// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationTest : ModuleRules
	{
		public AutomationTest(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"SourceControl",
					});
			}
		}
	}
}
