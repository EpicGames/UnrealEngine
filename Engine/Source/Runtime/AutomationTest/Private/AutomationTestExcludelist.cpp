// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationTestExcludelist.h"

#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationTestExcludelist)

#if WITH_EDITOR
	#include "HAL/PlatformFileManager.h"
	#include "ISourceControlOperation.h"
	#include "SourceControlOperations.h"
	#include "ISourceControlProvider.h"
	#include "ISourceControlModule.h"
	#include "GenericPlatform/GenericPlatformFile.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAutomationTestExcludelist, Log, All);

namespace
{
	const FString FunctionalTestsPreFix = "Project.Functional Tests.";

	void SortExcludelist(TMap<FString, FAutomationTestExcludelistEntry>& List)
	{
		// Sort in alphabetical order, shortest to longest of key property.
		// That is to naturally gives priority to parent suite over individual test exclusion when calling GetExcludeTestEntry(TestName).
		List.KeySort([](const FString& A, const FString& B)
			{
				return A < B;
			});
	}

	void SortExcludelist(TArray<FAutomationTestExcludelistEntry>& List)
	{
		// Sort in alphabetical order, shortest to longest of FullTestName property.
		// That is to naturally gives priority to parent suite over individual test exclusion when calling GetExcludeTestEntry(TestName).
		List.Sort([](const FAutomationTestExcludelistEntry& A, const FAutomationTestExcludelistEntry& B)
			{
				return A.FullTestName < B.FullTestName;
			});
	}

	const FString TicketTrackerURLHashtagPropertyName = "URLHashtag";
	const FString TicketTrackerURLBasePropertyName = "URLBase";
} // anonymous namespace

#if WITH_EDITOR
void FAutomationTestExcludeOptions::UpdateReason(const FString& BeautifiedReason, const FString& TaskTrackerTicketId)
{
	if (TaskTrackerTicketId.IsEmpty())
	{
		Reason = FName(BeautifiedReason);
	}
	else
	{
		static const UAutomationTestExcludelist* Excludelist = UAutomationTestExcludelist::Get();
		check(nullptr != Excludelist);

		FString FullTicketString = Excludelist->GetTaskTrackerTicketTag() + TEXT(" ") + TaskTrackerTicketId;

		if (BeautifiedReason.IsEmpty())
		{
			Reason = FName(FullTicketString);
		}
		else
		{
			const bool LastSymbolIsSpaceOrPunct =
			(
				TChar<FString::ElementType>::IsWhitespace(BeautifiedReason[BeautifiedReason.Len() - 1])
				|| TChar<FString::ElementType>::IsPunct(BeautifiedReason[BeautifiedReason.Len() - 1])
			);

			if (!LastSymbolIsSpaceOrPunct)
			{
				FullTicketString = TEXT(" ") + FullTicketString;
			}

			Reason = FName(BeautifiedReason + FullTicketString);
		}
	}
}
#endif // WITH_EDITOR

void FAutomationTestExcludelistEntry::Finalize()
{
	if (!IsEmpty())
	{
		return;
	}

	FString TestStr = Test.ToString().TrimStartAndEnd();

	bool IsFunctionalTest = TestStr.StartsWith(FunctionalTestsPreFix);

	// Backward compatibility - merge Map and Test properties of Functional Test
	FString MapStr = Map.ToString().TrimStartAndEnd();
	if (MapStr.StartsWith(TEXT("/")) && !IsFunctionalTest)
	{
		TestStr = FunctionalTestsPreFix + MapStr + TEXT(".") + TestStr;
		IsFunctionalTest = true;
	}
	// Backward compatibility - Convert package path by using dot syntax instead of /
	if (IsFunctionalTest)
	{
		TestStr = TestStr.Replace(TEXT("./Game/"), TEXT(".")).Replace(TEXT("/"), TEXT("."));
	}

	FullTestName = TestStr.ToLower();
}

FString FAutomationTestExcludelistEntry::GetStringForHash() const
{
	return FullTestName + Reason.ToString() + SetToString(RHIs) + (Warn? TEXT("1") : TEXT("0"));
}

UAutomationTestExcludelist* UAutomationTestExcludelist::Get()
{
	UAutomationTestExcludelist* Obj = GetMutableDefault<UAutomationTestExcludelist>();
	if (!Obj->DefaultConfig)
	{
		Obj->Initialize();
	}

	return Obj;
}

void UAutomationTestExcludelist::Initialize()
{
	DefaultConfig = GetMutableDefault<UAutomationTestExcludelistConfig>();
	check(nullptr != DefaultConfig);

	if (PlatformConfigs.IsEmpty())
	{
		LoadPlatformConfigs();
		PopulateEntries();
	}

	DefaultConfig->LoadTaskTrackerProperties();
}

void UAutomationTestExcludelist::LoadPlatformConfigs()
{
	PlatformConfigs.Empty();

	for (auto* PlatformSettings : AutomationTestPlatform::GetAllPlatformsSettings(UAutomationTestExcludelistConfig::StaticClass()))
	{
		PlatformConfigs.Emplace(PlatformSettings->GetPlatformName(), CastChecked<UAutomationTestExcludelistConfig>(PlatformSettings));
	}
}

void UAutomationTestExcludelist::PopulateEntries()
{
	LLM_SCOPE_BYNAME(TEXT("AutomationTest/Settings"));
	Entries.Empty();

	// Populate with default first
	for (const FAutomationTestExcludelistEntry& Entry : DefaultConfig->GetEntries())
	{
		Entries.Emplace(Entry.FullTestName, Entry);
	}
	// Merge platforms with default config
	for (auto& Config : PlatformConfigs)
	{
		for (const FAutomationTestExcludelistEntry& PlatformEntry : Config.Value->GetEntries())
		{
			if (FAutomationTestExcludelistEntry* Entry = Entries.Find(PlatformEntry.FullTestName))
			{
				if (Entry->Platforms.IsEmpty())
				{
					continue;
				}

				Entry->Platforms.Add(Config.Key);
				Entry->RHIs.Append(PlatformEntry.RHIs);
			}
			else
			{
				Entries.Emplace(
					PlatformEntry.FullTestName,
					PlatformEntry
				).Platforms.Add(Config.Key);
			}
		}
	}
	SortExcludelist(Entries);
}

void UAutomationTestExcludelist::AddToExcludeTest(const FString& TestName, const FAutomationTestExcludelistEntry& ExcludelistEntry)
{
	auto NewEntry = FAutomationTestExcludelistEntry(ExcludelistEntry);
	NewEntry.Test = *(TestName.TrimStartAndEnd());
	if (!NewEntry.Map.IsNone())
	{
		NewEntry.Map = TEXT("");
	}
	NewEntry.Finalize();
	Entries.Emplace(NewEntry.FullTestName, NewEntry);
	SortExcludelist(Entries);
}

void UAutomationTestExcludelist::RemoveFromExcludeTest(const FString& TestName)
{
	if (TestName.IsEmpty())
		return;

	Entries.Remove(TestName.TrimStartAndEnd().ToLower());
	SortExcludelist(Entries);
}

bool UAutomationTestExcludelist::IsTestExcluded(const FString& TestName) const
{
	static const FName None;
	static const TSet<FName> EmptySet;
	return IsTestExcluded(TestName, None, EmptySet, nullptr, nullptr);
}

bool UAutomationTestExcludelist::IsTestExcluded(const FString& TestName, const TSet<FName>& RHI, FName* OutReason, bool* OutWarn) const
{
	return IsTestExcluded(TestName, FPlatformProperties::IniPlatformName(), RHI, OutReason, OutWarn);
}

bool UAutomationTestExcludelist::IsTestExcluded(const FString & TestName, const FName& Platform, const TSet<FName>& RHI, FName * OutReason, bool* OutWarn) const
{
	if (const auto Entry = GetExcludeTestEntry(TestName, Platform, RHI))
	{
		if (OutReason != nullptr)
		{
			*OutReason = Entry->Reason;
		}

		if (OutWarn != nullptr)
		{
			*OutWarn = Entry->Warn;
		}

		return true;
	}

	return false;
}

FString UAutomationTestExcludelist::GetConfigFilename() const
{
	return DefaultConfig->GetConfigFilename();
}

FString UAutomationTestExcludelist::GetConfigFilenameForEntry(const FAutomationTestExcludelistEntry& Entry) const
{
	return GetConfigFilenameForEntry(Entry, FPlatformProperties::IniPlatformName());
}

FString UAutomationTestExcludelist::GetConfigFilenameForEntry(const FAutomationTestExcludelistEntry& Entry, const FName& PlatformName) const
{
	if (Entry.Platforms.IsEmpty())
	{
		return DefaultConfig->GetConfigFilename();
	}

	// Align with current platform
	if (const TObjectPtr<UAutomationTestExcludelistConfig>* ConfigPtr = PlatformConfigs.Find(PlatformName))
	{
		return (*ConfigPtr)->GetConfigFilename();
	}

	// Otherwise take the first item from the entry platform list
	FName FirstItem = *Entry.Platforms.FindArbitraryElement();
	if (const TObjectPtr<UAutomationTestExcludelistConfig>* ConfigPtr = PlatformConfigs.Find(FirstItem))
	{
		return (*ConfigPtr)->GetConfigFilename();
	}

	return TEXT("");
}

FString UAutomationTestExcludelist::GetTaskTrackerURLBase() const
{
	return DefaultConfig->GetTaskTrackerURLBase();
}

FString UAutomationTestExcludelist::GetConfigTaskTrackerHashtag() const
{
	return DefaultConfig->GetTaskTrackerURLHashtag();
}

FString UAutomationTestExcludelist::GetBeautifiedTaskTrackerTicketTagSuffix() const
{
	static const FString DefaultTaskTrackerTagSuffix = "unknown";

	FString TaskTrackerTicketTagSuffix = DefaultConfig->GetTaskTrackerURLHashtag();
	TaskTrackerTicketTagSuffix.TrimStartAndEndInline();

	if (TaskTrackerTicketTagSuffix.IsEmpty())
	{
		TaskTrackerTicketTagSuffix = DefaultTaskTrackerTagSuffix;
	}

	return TaskTrackerTicketTagSuffix;
}

FString UAutomationTestExcludelist::GetTaskTrackerName() const
{
	FString TaskTrackerName = GetBeautifiedTaskTrackerTicketTagSuffix();

	// Capitalize the first letter
	TaskTrackerName[0] = TChar<FString::ElementType>::ToUpper(TaskTrackerName[0]);

	return TaskTrackerName;
}

FString UAutomationTestExcludelist::GetTaskTrackerTicketTag() const
{
	return (TEXT("#") + GetBeautifiedTaskTrackerTicketTagSuffix());
}

void UAutomationTestExcludelist::SaveToConfigs()
{
	// Reset the cached configs
	DefaultConfig->Reset();
	for (auto& Config : PlatformConfigs)
	{
		Config.Value->Reset();
	}

	// Populate configs
	for (auto& EntryPair : Entries)
	{
		if (EntryPair.Value.Platforms.IsEmpty())
		{
			DefaultConfig->AddEntry(EntryPair.Value);
		}
		else
		{
			FAutomationTestExcludelistEntry PlatformEntry = EntryPair.Value;
			for (const FName& PlatformName : EntryPair.Value.Platforms)
			{
				if (!EntryPair.Value.RHIs.IsEmpty())
				{
					// Filter in only the RHIs that are relevant for the platform
					PlatformEntry.RHIs = EntryPair.Value.RHIs.Intersect(FAutomationTestExcludeOptions::GetPlatformRHIOptionNamesFromSettings(PlatformName));
				}
				if (TObjectPtr<UAutomationTestExcludelistConfig>* ConfigPtr = PlatformConfigs.Find(PlatformName))
				{
					(*ConfigPtr)->AddEntry(PlatformEntry);
				}
				else
				{
					UAutomationTestExcludelistConfig* Config = CastChecked<UAutomationTestExcludelistConfig>(UAutomationTestPlatformSettings::Create(UAutomationTestExcludelistConfig::StaticClass(), PlatformName.ToString()));
					Config->AddEntry(PlatformEntry);
					PlatformConfigs.Emplace(PlatformName, Config);
				}
			}
		}
	}

	// Save the configs
	DefaultConfig->SaveConfig();
	for (auto& Config : PlatformConfigs)
	{
		Config.Value->SaveConfig();
	}
}

const FAutomationTestExcludelistEntry* UAutomationTestExcludelist::GetExcludeTestEntry(const FString& TestName) const
{
	static const FName None;
	static const TSet<FName> EmptySet;
	return GetExcludeTestEntry(TestName, None, EmptySet);
}

const FAutomationTestExcludelistEntry* UAutomationTestExcludelist::GetExcludeTestEntry(const FString& TestName, const TSet<FName>& RHI) const
{
	return GetExcludeTestEntry(TestName, FPlatformProperties::IniPlatformName(), RHI);
}

const FAutomationTestExcludelistEntry* UAutomationTestExcludelist::GetExcludeTestEntry(const FString& TestName, const FName& Platform, const TSet<FName>& RHI) const
{
	if (TestName.IsEmpty())
		return nullptr;

	const FString NameToCompare = TestName.TrimStartAndEnd().ToLower();

	const FAutomationTestExcludelistEntry* OutEntry = nullptr;

	for (auto& EntryPair : Entries)
	{
		if (NameToCompare.StartsWith(EntryPair.Key))
		{
			if (NameToCompare.Len() == EntryPair.Key.Len() || NameToCompare.Mid(EntryPair.Key.Len(), 1) == TEXT("."))
			{
				if (!Platform.IsNone() && !EntryPair.Value.Platforms.IsEmpty() && !EntryPair.Value.Platforms.Contains(Platform))
				{
					continue;
				}

				if (EntryPair.Value.RHIs.IsEmpty())
				{
					return &EntryPair.Value;
				}
				if (RHI.IsEmpty())
				{
					OutEntry = &EntryPair.Value;
					continue;
				}
				const int8 IntersectNum = RHI.Intersect(EntryPair.Value.RHIs).Num();
				if (IntersectNum > 0 && IntersectNum == EntryPair.Value.NumRHIType())
				{
					return &EntryPair.Value;
				}
			}
		}
	}

	return OutEntry;
}

void UAutomationTestExcludelistConfig::InitializeSettingsDefault()
{
	Reset();
}

void UAutomationTestExcludelistConfig::Reset()
{
	ExcludeTest.Empty();
	EntriesHash = FSHAHash();
}

void UAutomationTestExcludelistConfig::AddEntry(const FAutomationTestExcludelistEntry& Entry)
{
	ExcludeTest.Add(Entry);
	UpdateHash(Entry);
}

void UAutomationTestExcludelistConfig::UpdateHash(const FAutomationTestExcludelistEntry& Entry)
{
	check(!Entry.IsEmpty());
	FSHA1 SHA;
	SHA.Update((const uint8*)&EntriesHash, sizeof(EntriesHash));
	FString EntryString = Entry.GetStringForHash();
	SHA.UpdateWithString(*EntryString, EntryString.Len());
	EntriesHash = SHA.Finalize();
}

const TArray<FAutomationTestExcludelistEntry>& UAutomationTestExcludelistConfig::GetEntries() const
{
	return ExcludeTest;
}

void UAutomationTestExcludelistConfig::PostInitProperties()
{
	Super::PostInitProperties();

	for (auto& Entry : ExcludeTest)
	{
		Entry.Finalize();
	}
	SortExcludelist(ExcludeTest);
	// Hashing is order sensitive and depends on the Entry being finalized.
	for (auto& Entry : ExcludeTest)
	{
		UpdateHash(Entry);
	}
	// Store the initial hash to detect dirty state.
	SavedEntriesHash = EntriesHash;
}

#if WITH_EDITOR
bool CheckOutOrAddFile(const FString& InFileToCheckOut)
{
	bool bSuccessfullyCheckedOutOrAddedFile = false;
	if (ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(InFileToCheckOut, EStateCacheUsage::Use);

		TArray<FString> FilesToBeCheckedOut;
		FilesToBeCheckedOut.Add(InFileToCheckOut);

		if (SourceControlState.IsValid())
		{
			if (SourceControlState->IsSourceControlled())
			{
				if (SourceControlState->IsDeleted())
				{
					UE_LOGF(LogAutomationTestExcludelist, Error, "The configuration file is marked for deletion.");
				}
				else if (SourceControlState->CanCheckout() || SourceControlState->IsCheckedOutOther() || FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*InFileToCheckOut))
				{
					ECommandResult::Type CommandResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToBeCheckedOut);
					if (CommandResult == ECommandResult::Failed)
					{
						UE_LOGF(LogAutomationTestExcludelist, Error, "Failed to check out the configuration file.");
					}
					else if (CommandResult == ECommandResult::Cancelled)
					{
						UE_LOGF(LogAutomationTestExcludelist, Warning, "Checkout was cancelled.");
					}
					else
					{
						bSuccessfullyCheckedOutOrAddedFile = true;
					}
				}
				else if (SourceControlState->CanAdd() || SourceControlState->IsUnknown())
				{
					ECommandResult::Type CommandResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToBeCheckedOut);
					if (CommandResult == ECommandResult::Failed)
					{
						UE_LOGF(LogAutomationTestExcludelist, Error, "Failed to mark for add the configuration file.");
					}
					else if (CommandResult == ECommandResult::Cancelled)
					{
						UE_LOGF(LogAutomationTestExcludelist, Warning, "Mark for add was cancelled.");
					}
					else
					{
						bSuccessfullyCheckedOutOrAddedFile = true;
					}
				}
				else if (SourceControlState->IsAdded())
				{
					bSuccessfullyCheckedOutOrAddedFile = true;
				}
			}
			else if (!SourceControlState->IsUnknown())
			{
				if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*InFileToCheckOut))
				{
					return true;
				}

				ECommandResult::Type CommandResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToBeCheckedOut);

				if (CommandResult == ECommandResult::Failed)
				{
					UE_LOGF(LogAutomationTestExcludelist, Error, "Failed to check out the configuration file.");
				}
				else if (CommandResult == ECommandResult::Cancelled)
				{
					UE_LOGF(LogAutomationTestExcludelist, Warning, "Checkout was cancelled..");
				}
				else
				{
					bSuccessfullyCheckedOutOrAddedFile = true;
				}
			}
		}
	}
	return bSuccessfullyCheckedOutOrAddedFile;
}

bool MakeWritable(const FString& InFileToMakeWritable)
{
	if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*InFileToMakeWritable))
	{
		return true;
	}

	return FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*InFileToMakeWritable, false);
}
#endif

void UAutomationTestExcludelistConfig::SaveConfig()
{
	// Exit early if entries has not changed.
	if (SavedEntriesHash == EntriesHash)
	{
		return;
	}

#if WITH_EDITOR
	FString ConfigFilename = GetConfigFilename();
	bool bIsFileExists = FPlatformFileManager::Get().GetPlatformFile().FileExists(*ConfigFilename);
	bool bIsWritable = bIsFileExists && !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*ConfigFilename);
	if (bIsFileExists && !bIsWritable)
	{
		bIsWritable = CheckOutOrAddFile(ConfigFilename);
		if (!bIsWritable)
		{
			UE_LOGF(LogAutomationTestExcludelist, Warning, "Config file '%ls' is readonly and could not be checked out. File will be marked writable.", *ConfigFilename);
			bIsWritable = MakeWritable(ConfigFilename);
		}
	}

	if (bIsFileExists && !bIsWritable)
	{
		UE_LOGF(LogAutomationTestExcludelist, Error, "Failed to make the configuration file '%ls' writable.", *ConfigFilename);
	}
	else
#endif
	{
		if (UObject::TryUpdateDefaultConfigFile())
		{
			SavedEntriesHash = EntriesHash;
#if WITH_EDITOR
			if (!bIsFileExists)
			{
				CheckOutOrAddFile(ConfigFilename);
			}
#endif
		}
	}
}

void UAutomationTestExcludelistConfig::LoadTaskTrackerProperties()
{
	GConfig->GetString(*GetSectionName(), *TicketTrackerURLHashtagPropertyName, TaskTrackerURLHashtag, GEngineIni);
	GConfig->GetString(*GetSectionName(), *TicketTrackerURLBasePropertyName, TaskTrackerURLBase, GEngineIni);
}


