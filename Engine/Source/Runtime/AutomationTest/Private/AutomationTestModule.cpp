// Copyright Epic Games, Inc. All Rights Reserved.
#include "AutomationTestModule.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogAutomationTest)

/**
 * Implements the AutomationTesting module.
 */
class FAutomationTestModule
	: public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}
};


IMPLEMENT_MODULE(FAutomationTestModule, AutomationTest);
