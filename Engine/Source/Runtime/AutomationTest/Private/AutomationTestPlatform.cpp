// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationTestPlatform.h"

#include "UObject/Package.h"

#if WITH_EDITOR
#include "Misc/CoreMisc.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationTestPlatform)

namespace AutomationTestPlatform
{
	const TSet<FName>& GetAllAvailablePlatformNames()
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Settings"));
		static TSet<FName> NameSet;
		if (NameSet.IsEmpty())
		{
#if WITH_EDITOR
			FName TargetPlatformName;
			const TMap<FName, FDataDrivenPlatformInfo>& PlatformInfos{ FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos() };
			for (const TPair<FName, FDataDrivenPlatformInfo>& PlatformInfosMapItem : PlatformInfos)
			{
				TargetPlatformName = PlatformInfosMapItem.Key;
				if (TargetPlatformName.IsNone() || FDataDrivenPlatformInfoRegistry::IsPlatformHiddenFromUI(TargetPlatformName))
				{
					continue;
				}
				const FDataDrivenPlatformInfo& PlatformInfo = PlatformInfosMapItem.Value;
				if (PlatformInfo.bIsFakePlatform || (!PlatformInfo.bEnabledForUse))
				{
					continue;
				}
				NameSet.Add(TargetPlatformName);
			}
			NameSet.Sort([](const FName& A, const FName& B) { return A.ToString() < B.ToString(); });
#else
			NameSet.Add(FPlatformProperties::IniPlatformName());
#endif
		}

		return NameSet;
	}

	TArray<UAutomationTestPlatformSettings*>& GetAllPlatformsSettings(TSubclassOf<UAutomationTestPlatformSettings> SettingsClass)
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Settings"));
		static TMap<FString, TArray<UAutomationTestPlatformSettings*>> Settings;
		const FString SettingsName = SettingsClass->GetName();

		if (!Settings.Contains(SettingsName))
		{
			const TSet<FName> Platforms = GetAllAvailablePlatformNames();
			TArray<UAutomationTestPlatformSettings*> PlatformCollection;

			for (const FName& PlatformName : Platforms)
			{
				UAutomationTestPlatformSettings* Config = UAutomationTestPlatformSettings::Create(SettingsClass, PlatformName.ToString());
				check(Config != nullptr);
				Config->LoadConfig();
				Config->AddToRoot();
				PlatformCollection.Add(Config);
			}

			Settings.Emplace(SettingsName, PlatformCollection);
		}

		return *Settings.Find(SettingsName);
	}
}

UAutomationTestPlatformSettings* UAutomationTestPlatformSettings::Create(TSubclassOf<UAutomationTestPlatformSettings> SettingsClass, const FString InPlatformName)
{
	const FString PlatformConfigsName = SettingsClass->GetName() + TEXT("_") + InPlatformName;
	UAutomationTestPlatformSettings* Config = NewObject<UAutomationTestPlatformSettings>(GetTransientPackage(), SettingsClass, FName(*PlatformConfigsName));
	Config->PlatformName = InPlatformName;
	Config->InitializeSettingsDefault();
	return Config;
}

void UAutomationTestPlatformSettings::OverrideConfigSection(FString& SectionName)
{
	SectionName = GetSectionName();
	if (!PlatformName.IsEmpty())
	{
		SectionName += TEXT(":");
		SectionName += PlatformName;
	}
}

const TCHAR* UAutomationTestPlatformSettings::GetConfigOverridePlatform() const
{
	return PlatformName.IsEmpty() ? nullptr : *PlatformName;
}

FString UAutomationTestPlatformSettings::GetConfigFilename() const
{
	return UObject::GetDefaultConfigFilename();
}

const FString& UAutomationTestPlatformSettings::GetPlatformName() const
{
	return PlatformName;
}

void UAutomationTestPlatformSettings::LoadConfig()
{
	UObject::LoadConfig(GetClass());
	PostInitProperties();
}
