// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"

#include "AutomationState.generated.h"


/** Enumeration of unit test status for special dialog */
UENUM()
enum class EAutomationState : uint8
{
	NotRun,					// Automation test was not run
	InProcess,				// Automation test is running now
	Fail,					// Automation test was run and failed
	Success,				// Automation test was run and succeeded
	Skipped,				// Automation test was skipped
};


inline const FString AutomationStateToString(EAutomationState InValue)
{
	return UEnum::GetDisplayValueAsText(InValue).ToString();
}
