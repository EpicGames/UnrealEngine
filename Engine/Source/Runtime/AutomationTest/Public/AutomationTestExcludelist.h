// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutomationTestPlatform.h"
#include "CoreMinimal.h"
#include "Misc/SecureHash.h"

#include "AutomationTestExcludelist.generated.h"

UENUM()
enum class ETEST_RHI_Options
{
	DirectX11,
	DirectX12,
	Vulkan,
	Metal,
	Null
};

inline FString LexToString(ETEST_RHI_Options Option)
{
	switch (Option)
	{
	case ETEST_RHI_Options::DirectX11:    return TEXT("DirectX 11");
	case ETEST_RHI_Options::DirectX12:    return TEXT("DirectX 12");
	case ETEST_RHI_Options::Vulkan:       return TEXT("Vulkan");
	case ETEST_RHI_Options::Metal:        return TEXT("Metal");
	case ETEST_RHI_Options::Null:         return TEXT("Null");
	default:                              return TEXT("Unknown");
	}
}

UENUM()
enum class ETEST_RHI_FeatureLevel_Options
{
	SM5,
	SM6
};

inline FString LexToString(ETEST_RHI_FeatureLevel_Options Option)
{
	switch (Option)
	{
	case ETEST_RHI_FeatureLevel_Options::SM5:   return TEXT("SM5");
	case ETEST_RHI_FeatureLevel_Options::SM6:   return TEXT("SM6");
	default:                                    return TEXT("Unknown");
	}
}

inline FString SetToString(const TSet<FName>& Set)
{
	if (Set.IsEmpty())
	{
		return TEXT("");
	}

	TArray<FString> List;
	for (auto& Item : Set)
	{
		List.Add(Item.ToString());
	}
	List.Sort();

	return FString::Join(List, TEXT(", "));
}

inline FString SetToShortString(const TSet<FName>& Set)
{
	if (Set.IsEmpty())
	{
		return TEXT("");
	}

	TArray<FString> List;
	for (auto& Item : Set)
	{
		List.Add(Item.ToString());
	}
	if (List.Num() == 1)
	{
		return List[0];
	}		
	List.Sort();

	return List[0] + TEXT("+");
}

UCLASS(config = Engine, defaultconfig, MinimalAPI)
class UAutomationTestExcludelistSettings : public UAutomationTestPlatformSettings
{
	GENERATED_BODY()

public:

	UPROPERTY(Config)
	TArray<FName> SupportedRHIs;

protected:
	virtual void InitializeSettingsDefault() { }
	virtual FString GetSectionName() { return TEXT("AutomationTestExcludelistSettings"); }

};

USTRUCT()
struct FAutomationTestExcludeOptions
{
	GENERATED_BODY()

	template<typename EnumType>
	static const TSet<FName>& GetAllRHIOptionNames()
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Settings"));
		static TSet<FName> NameSet;
		if (NameSet.IsEmpty())
		{
			if constexpr (std::is_same_v<EnumType, ETEST_RHI_Options> || std::is_same_v<EnumType, ETEST_RHI_FeatureLevel_Options>)
			{
				UEnum* Enum = StaticEnum<EnumType>();
				int32 Num_Flags = Enum->NumEnums() - 1;
				for (int32 i = 0; i < Num_Flags; i++)
				{
					NameSet.Add(*LexToString((EnumType)Enum->GetValueByIndex(i)));
				}
			}
		}

		return NameSet;
	}

	static const TSet<FName>& GetAllRHIOptionNamesFromSettings()
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Settings"));
		static TSet<FName> NameSet;
		if (NameSet.IsEmpty())
		{
			for (auto Settings : AutomationTestPlatform::GetAllPlatformsSettings(UAutomationTestExcludelistSettings::StaticClass()))
			{
				NameSet.Append(CastChecked<UAutomationTestExcludelistSettings>(Settings)->SupportedRHIs);
			}
			NameSet.Sort([](const FName& A, const FName& B) { return A.ToString() < B.ToString(); });
		}

		return NameSet;
	}

	static const TSet<FName>& GetPlatformRHIOptionNamesFromSettings(const FName& Platform)
	{
		LLM_SCOPE_BYNAME(TEXT("AutomationTest/Settings"));
		static TMap<FName, TSet<FName>> PlatformSettings;
		if (PlatformSettings.IsEmpty())
		{
			for (auto Settings : AutomationTestPlatform::GetAllPlatformsSettings(UAutomationTestExcludelistSettings::StaticClass()))
			{
				PlatformSettings.Emplace(Settings->GetPlatformName()).Append(CastChecked<UAutomationTestExcludelistSettings>(Settings)->SupportedRHIs);
			}
		}

		return PlatformSettings.FindOrAdd(Platform);
	}

#if WITH_EDITOR
	AUTOMATIONTEST_API void UpdateReason(const FString& BeautifiedReason, const FString& TaskTrackerTicketId);
#endif //WITH_EDITOR

	/* Name of the target test */
	UPROPERTY(VisibleAnywhere, Category = ExcludeTestOptions)
	FName Test;

	/* Reason to why the test is excluded */
	UPROPERTY(EditAnywhere, Category = ExcludeTestOptions)
	FName Reason;

	/* Options to target specific RHI. No option means it should be applied to all RHIs */
	UPROPERTY(EditAnywhere, Category = ExcludeTestOptions)
	TSet<FName> RHIs;

	/* Options to target specific platform. No option means it should be applied to all platforms */
	UPROPERTY(EditAnywhere, Category = ExcludeTestOptions)
	TSet<FName> Platforms;

	/* Should the Reason be reported as a warning in the log */
	UPROPERTY(EditAnywhere, Category = ExcludeTestOptions)
	bool Warn = false;
};

USTRUCT()
struct FAutomationTestExcludelistEntry
{
	GENERATED_BODY()

	FAutomationTestExcludelistEntry() { }

	FAutomationTestExcludelistEntry(const FAutomationTestExcludeOptions& Options)
		: Platforms(Options.Platforms)
		, Test(Options.Test)
		, Reason(Options.Reason)
		, RHIs(Options.RHIs)
		, Warn(Options.Warn)
	{ }

	TSharedPtr<FAutomationTestExcludeOptions> GetOptions() const
	{
		TSharedPtr<FAutomationTestExcludeOptions> Options = MakeShareable(new FAutomationTestExcludeOptions());
		Options->Test = Test;
		Options->Reason = Reason;
		Options->Warn = Warn;
		Options->RHIs = RHIs;
		Options->Platforms = Platforms;
		
		return Options;
	}

	/* Return the number of RHI types that needs to be matched for exclusion */
	int8 NumRHIType() const
	{
		int8 Num = 0;
		// Test mentions of each RHI option types
		static const TSet<FName> AllRHI_OptionNames = FAutomationTestExcludeOptions::GetAllRHIOptionNamesFromSettings();
		if (RHIs.Difference(AllRHI_OptionNames).Num() < RHIs.Num())
		{
			Num++;
		}
		static const TSet<FName> AllRHI_FeatureLevel_OptionNames = FAutomationTestExcludeOptions::GetAllRHIOptionNames<ETEST_RHI_FeatureLevel_Options>();
		if (RHIs.Difference(AllRHI_FeatureLevel_OptionNames).Num() < RHIs.Num())
		{
			Num++;
		}

		return Num;
	}

	/* Determine if exclusion entry is propagated based on test name - used for management in test automation window */
	void SetPropagation(const FString& ForTestName)
	{
		bIsPropagated = FullTestName != ForTestName.TrimStartAndEnd().ToLower();
	}

	/* Return true if the entry is not specific */
	bool IsEmpty() const
	{
		return FullTestName.IsEmpty();
	}

	/* Reset entry to be un-specific */
	void Reset()
	{
		FullTestName.Empty();
		bIsPropagated = false;
	}

	/* Make the entry specific */
	void Finalize();

	/* Output string used to generate hash */
	FString GetStringForHash() const;

	/* Has conditional exclusion */
	bool HasConditions() const
	{
		return !RHIs.IsEmpty() || !Platforms.IsEmpty();
	}

	/* Remove overlapping exclusion conditions, return true if at least one condition was removed */
	bool RemoveConditions(const FAutomationTestExcludelistEntry& Entry)
	{
		if (!Entry.HasConditions())
		{
			return false;
		}

		bool GotRemoved = false;
		int Length;
		// Check RHIs
		Length = RHIs.Num();
		if (Length > 0)
		{
			RHIs = RHIs.Difference(Entry.RHIs);
			GotRemoved = GotRemoved || Length != RHIs.Num();
		}
		// Check Platforms
		Length = Platforms.Num();
		if (Length > 0)
		{
			Platforms = Platforms.Difference(Entry.Platforms);
			GotRemoved = GotRemoved || Length != Platforms.Num();
		}

		return GotRemoved;
	}

	/* Hold full test name/path */
	FString FullTestName;
	/* Is the entry comes from propagation */
	bool bIsPropagated = false;
	/* Platforms to which the entry applies */
	TSet<FName> Platforms;

	// Use FName instead of FString to read params from config that aren't wrapped with quotes

	/* Hold the name of the target functional test map */
	UPROPERTY(Config)
	FName Map;

	/* Hold the name of the target test - full test name is require here unless for functional tests */
	UPROPERTY(Config)
	FName Test;

	/* Reason to why the test is excluded */
	UPROPERTY(Config)
	FName Reason;

	/* Option to target specific RHI. Empty array means it should be applied to all RHI */
	UPROPERTY(Config)
	TSet<FName> RHIs;

	/* Should the Reason be reported as a warning in the log */
	UPROPERTY(Config)
	bool Warn = false;
};

UCLASS(config = Engine, defaultconfig, MinimalAPI)
class UAutomationTestExcludelistConfig : public UAutomationTestPlatformSettings
{
	GENERATED_BODY()

public:
	void Reset();
	void AddEntry(const FAutomationTestExcludelistEntry& Entry);
	const TArray<FAutomationTestExcludelistEntry>& GetEntries() const;
	FString GetTaskTrackerURLHashtag() const { return TaskTrackerURLHashtag; }
	FString GetTaskTrackerURLBase() const { return TaskTrackerURLBase; }

	void SaveConfig();

	void LoadTaskTrackerProperties();

protected:
	virtual void PostInitProperties() override;
	virtual void InitializeSettingsDefault() override;
	virtual FString GetSectionName() override { return TEXT("AutomationTestExcludelist"); }

private:
	void UpdateHash(const FAutomationTestExcludelistEntry& Entry);

	UPROPERTY(Transient)
	FString TaskTrackerURLHashtag;

	UPROPERTY(Transient)
	FString TaskTrackerURLBase;

	UPROPERTY(Config)
	TArray<FAutomationTestExcludelistEntry> ExcludeTest;

	FString PlatformName;
	/** Keep track of any changes */
	FSHAHash EntriesHash;
	/** Hash of what was last saved */
	FSHAHash SavedEntriesHash;
};

UCLASS(MinimalAPI)
class UAutomationTestExcludelist : public UObject
{
	GENERATED_BODY()

public:
	UAutomationTestExcludelist() { };

	static AUTOMATIONTEST_API UAutomationTestExcludelist* Get();

	/**
	 * Add an Entry to the exclusion list for TestName
	 *
	 * @param TestName The full test name to be added.
	 * @param Entry The FAutomationTestExcludelistEntry to be excluded.
	*/
	AUTOMATIONTEST_API void AddToExcludeTest(const FString& TestName, const FAutomationTestExcludelistEntry& ExcludelistEntry);

	/**
	 * Remove the Entry from the exclusion list for TestName
	 *
	 * @param TestName The full test name to be removed.
	*/
	AUTOMATIONTEST_API void RemoveFromExcludeTest(const FString& TestName);

	/**
	 * Is the TestName excluded?
	 *
	 * @param TestName The full test name to be tested.
	 * @return True if the test is excluded
	*/
	AUTOMATIONTEST_API bool IsTestExcluded(const FString& TestName) const;

	/**
	 * Is the TestName excluded?
	 *
	 * @param TestName The full test name to be tested.
	 * @param RHI The RHI the test is being evaluated against.
	 * @param OutReason The reason why it is excluded.
	 * @param OutWarn Whether a warning is going to be returned if the test is to be excluded.
	 * @return True if the test is excluded
	*/
	AUTOMATIONTEST_API bool IsTestExcluded(const FString& TestName, const TSet<FName>& RHI, FName* OutReason, bool* OutWarn) const;

	/**
	 * Is the TestName excluded?
	 *
	 * @param TestName The full test name to be tested.
	 * @param Platform The Platform the test is being evaluated against.
	 * @param RHI The RHI the test is being evaluated against.
	 * @param OutReason The reason why it is excluded.
	 * @param OutWarn Whether a warning is going to be returned if the test is to be excluded.
	 * @return True if the test is excluded
	*/
	AUTOMATIONTEST_API bool IsTestExcluded(const FString& TestName, const FName& Platform, const TSet<FName>& RHI, FName* OutReason, bool* OutWarn) const;

	/**
	 * Get the Entry object of the TestName if excluded
	 *
	 * @param TestName The full test name to be tested.
	 * @return Return the corresponding FAutomationTestExcludelistEntry if excluded otherwise nullptr
	*/
	AUTOMATIONTEST_API const FAutomationTestExcludelistEntry* GetExcludeTestEntry(const FString& TestName) const;

	/**
	 * Get the Entry object of the TestName if excluded
	 *
	 * @param TestName The full test name to be tested.
	 * @param Platform The Platform the test is being evaluated against.
	 * @return Return the corresponding FAutomationTestExcludelistEntry if excluded otherwise nullptr
	*/
	AUTOMATIONTEST_API const FAutomationTestExcludelistEntry* GetExcludeTestEntry(const FString& TestName, const TSet<FName>& RHI) const;

	/**
	 * Get the Entry object of the TestName if excluded
	 *
	 * @param TestName The full test name to be tested.
	 * @param Platform The Platform the test is being evaluated against.
	 * @param RHI The RHI the test is being evaluated against.
	 * @return Return the corresponding FAutomationTestExcludelistEntry if excluded otherwise nullptr
	*/
	AUTOMATIONTEST_API const FAutomationTestExcludelistEntry* GetExcludeTestEntry(const FString& TestName, const FName& Platform, const TSet<FName>& RHI) const;

	UE_DEPRECATED(5.4, "Please use GetConfigFilenameForEntry instead.")
	AUTOMATIONTEST_API FString GetConfigFilename() const;

	/**
	 * Get the path to the config from which the exclusion was defined from
	 *
	 * @param Entry The FAutomationTestExcludelistEntry that is referenced.
	 * @return Return the path to the corresponding config
	*/
	AUTOMATIONTEST_API FString GetConfigFilenameForEntry(const FAutomationTestExcludelistEntry& Entry) const;

	/**
	 * Get the path to the config from which the exclusion was defined from
	 *
	 * @param Entry The FAutomationTestExcludelistEntry that is referenced.
	 * @param Platform The Platform the test is being evaluated against.
	 * @return Return the path to the corresponding config
	*/
	AUTOMATIONTEST_API FString GetConfigFilenameForEntry(const FAutomationTestExcludelistEntry& Entry, const FName& Platform) const;

	/**
	 * Get URL base that is used while restoring a task tracker task's URL.
	 *
	 * @return Return the URL base.
	*/
	AUTOMATIONTEST_API FString GetTaskTrackerURLBase() const;

	/**
	 * Get non-beautified task tracker's hashtag suffix (from the loaded configs).
	 *
	 * @return Return the string that contains non-beautified task tracker's hashtag suffix.
	 */
	AUTOMATIONTEST_API FString GetConfigTaskTrackerHashtag() const;

	/**
	 * Get Task tracker's name if task tracker information is configured.
	 *
	 * @return Return task tracker's name if it is configured or empty string otherwise.
	*/
	AUTOMATIONTEST_API FString GetTaskTrackerName() const;

	/**
	 * Get hashtag that is used for referencing tasks if task tracker information is configured.
	 *
	 * @return Return hashtag that is used for referencing task tracker tasks if it is configured or empty string otherwise.
	*/
	AUTOMATIONTEST_API FString GetTaskTrackerTicketTag() const;

	/** Save all the required configs */
	AUTOMATIONTEST_API void SaveToConfigs();

	UE_DEPRECATED(5.4, "Please use SaveToConfigs instead.")
	void SaveConfig() { SaveToConfigs(); }

private:
	/** Initiate the loading of the configs */
	void Initialize();

	/** Load all possible platform configs */
	void LoadPlatformConfigs();

	/** Populate the exclusion entries from the loaded configs */
	void PopulateEntries();

	/** Get beautified hashtag suffix (starts after leading#) that is used for referencing tasks if task tracker information is configured */
	FString GetBeautifiedTaskTrackerTicketTagSuffix() const;

	/** Exclusion entries */
	TMap<FString, FAutomationTestExcludelistEntry> Entries;

	UPROPERTY(Transient)
	TObjectPtr<UAutomationTestExcludelistConfig> DefaultConfig;

	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UAutomationTestExcludelistConfig>> PlatformConfigs;
};
