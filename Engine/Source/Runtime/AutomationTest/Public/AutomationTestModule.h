// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"

AUTOMATIONTEST_API DECLARE_LOG_CATEGORY_EXTERN(LogAutomationTest, Log, All);