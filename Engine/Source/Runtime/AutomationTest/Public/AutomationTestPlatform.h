// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"

#include "AutomationTestPlatform.generated.h"

UCLASS(Abstract, MinimalAPI)
class UAutomationTestPlatformSettings : public UObject
{
	GENERATED_BODY()

public:
	UAutomationTestPlatformSettings() {}

	AUTOMATIONTEST_API static UAutomationTestPlatformSettings* Create(TSubclassOf<UAutomationTestPlatformSettings> SettingsClass, const FString PlatformName);

	//~UObject interface
	virtual void OverrideConfigSection(FString& SectionName) override;
	virtual const TCHAR* GetConfigOverridePlatform() const override;
	//~End of UObject interface

	AUTOMATIONTEST_API void LoadConfig();
	AUTOMATIONTEST_API FString GetConfigFilename() const;
	AUTOMATIONTEST_API const FString& GetPlatformName() const;

protected:
	virtual void InitializeSettingsDefault() { }
	virtual FString GetSectionName() { return StaticClass()->GetPathName(); }

private:
	FString PlatformName;
};

namespace AutomationTestPlatform
{
	/** Return the names of all available platforms */
	AUTOMATIONTEST_API const TSet<FName>& GetAllAvailablePlatformNames();

	/** Load and return target automation test settings collection of all platforms  */
	AUTOMATIONTEST_API TArray<UAutomationTestPlatformSettings*>& GetAllPlatformsSettings(TSubclassOf<UAutomationTestPlatformSettings> SettingsClass);
};
